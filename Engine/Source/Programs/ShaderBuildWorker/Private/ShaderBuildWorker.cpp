// Copyright Epic Games, Inc. All Rights Reserved.

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "DerivedDataBuildFunction.h"
#include "DerivedDataBuildFunctionFactory.h"
#include "DerivedDataValue.h"
#include "Logging/StructuredLog.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"
#include "ShaderCompileWorkerUtil.h"
#include "SocketSubsystem.h"

class FCompileShaderJobsBuildFunction final : public UE::DerivedData::IBuildFunction
{
public:
	const UE::FUtf8SharedString& GetName() const final
	{
		static const UE::FUtf8SharedString Name(ANSITEXTVIEW("CompileShaderJobs"));
		return Name;
	}

	FGuid GetVersion() const final
	{
		// This version must match DerivedDataBuildControllerModule.cpp.
		// Version is not yet meaningfully derived from the shader build code,
		// which means that it cannot yet reliably distinguish two build workers.
		static const FGuid Version(TEXT("83027356-2cf7-41ca-aba5-c81ab0ff2129"));
		return Version;
	}

	void Configure(UE::DerivedData::FBuildConfigContext& Context) const final
	{
	}

	void Build(UE::DerivedData::FBuildContext& Context) const final;
};

void FCompileShaderJobsBuildFunction::Build(UE::DerivedData::FBuildContext& Context) const
{
	using namespace UE::DerivedData;

	FShaderCompileWorkerDiagnostics WorkerDiagnostics;

	int32 NumProcessedJobs = 0;
	TArray<FShaderCompileJob> SingleJobs;
	TArray<FShaderPipelineCompileJob> PipelineJobs;
	TArray<FString> PipelineJobNames;

	if (FShaderCompileWorkerUtil::GetShaderFormats().IsEmpty())
	{
		FSCWErrorCode::Report(FSCWErrorCode::NoTargetShaderFormatsFound, TEXT("No target shader formats found!"));
	}
	else
	{
		WorkerDiagnostics.EntryPointTimestamp = FPlatformTime::Seconds();

		if (FSharedBuffer Input = Context.FindInput(ANSITEXTVIEW("Input")))
		{
			const double BatchProcessStartTime = FPlatformTime::Seconds();
			WorkerDiagnostics.BatchPreparationTime = BatchProcessStartTime - WorkerDiagnostics.EntryPointTimestamp;

			FMemoryReaderView InputAr(Input, /*bIsPersistent*/ true);

			// TODO: Passed to SCW as -DebugSourceFiles=<Path1>[,<Path2>...] relative to WorkingDirectory
			TArray<FString> DebugSourceFiles;

			auto LoadVirtualSourceFile = [&Context](int32 VirtualSourceIndex) -> TArray<uint8>
			{
				TArray<uint8> Output;
				if (FSharedBuffer Data = Context.FindInput(WriteToAnsiString<16>(ANSITEXTVIEW("Virtual"), VirtualSourceIndex)))
				{
					Output.Append((const uint8*)Data.GetData(), (int32)Data.GetSize());
				}
				return Output;
			};

			FShaderCompileWorkerUtil::ProcessInputFromArchive(&InputAr, {}, DebugSourceFiles, LoadVirtualSourceFile,
				NumProcessedJobs, SingleJobs, PipelineJobs, PipelineJobNames);

			WorkerDiagnostics.BatchProcessTime = FPlatformTime::Seconds() - BatchProcessStartTime;
		}
		else
		{
			FSCWErrorCode::Report(FSCWErrorCode::BadInputFile, TEXT("Missing input file!"));
		}
	}

	WorkerDiagnostics.ErrorCode = FSCWErrorCode::IsSet() ? FSCWErrorCode::Get() : FSCWErrorCode::Success;

	static FString LocalHostName = []
	{
		FString Output;
		ISocketSubsystem::Get()->GetHostName(Output);
		return Output;
	}();

	TArray<uint8> MemBlock;
	FMemoryWriter MemWriter(MemBlock);
	FShaderCompileWorkerUtil::WriteToOutputArchive(MemWriter, LocalHostName, WorkerDiagnostics, NumProcessedJobs, SingleJobs, PipelineJobs, PipelineJobNames);

	static const FValueId OutputId = FValueId::FromName(ANSITEXTVIEW("Output"));
	Context.AddValue(OutputId, MakeSharedBufferFromArray(MoveTemp(MemBlock)));

	// Reset error code as it can receive a new value in subsequent builds.
	FSCWErrorCode::Reset();
}

static UE::DerivedData::TBuildFunctionFactory<FCompileShaderJobsBuildFunction> CompileShaderJobsFunction;

DERIVEDDATABUILDWORKER_API extern int32 BuildWorkerMain(int32 ArgC, TCHAR* ArgV[], const TCHAR* ExtraArgs, const TCHAR* LoadModules);

INT32_MAIN_INT32_ARGC_TCHAR_ARGV()
{
	return BuildWorkerMain(ArgC, ArgV, TEXT("-Unattended -ReduceThreadUsage -CpuProfilerTrace -NoCrashReports"), TEXT("ShaderPreprocessor"));
}
