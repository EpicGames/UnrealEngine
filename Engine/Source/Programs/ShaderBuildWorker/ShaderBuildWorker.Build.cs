// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class ShaderBuildWorker : ModuleRules
{
	public ShaderBuildWorker(ReadOnlyTargetRules Target) : base(Target)
	{
		bTreatAsEngineModule = true;

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"ApplicationCore",
				"Core",
				"DerivedDataBuildWorker",
				"DerivedDataCache",
				"Projects",
				"RenderCore",
				"RHI",
				"SandboxFile",
				"ShaderCompilerCommon",
				"Sockets",
				"TargetPlatform",
				"TraceLog",
			});

		if (Target.Platform == UnrealTargetPlatform.Win64)
		{
			RuntimeDependencies.Add(Path.Combine(Target.WindowsPlatform.DirectXDllDir, "d3dcompiler_47.dll"));
		}
	}
}
