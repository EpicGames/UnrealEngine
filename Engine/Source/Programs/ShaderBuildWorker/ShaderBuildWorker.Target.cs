// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.IO;
using UnrealBuildTool;

[SupportedPlatforms(UnrealPlatformClass.Editor)]
public class ShaderBuildWorkerTarget : DerivedDataBuildWorkerTarget
{
	public ShaderBuildWorkerTarget(TargetInfo Target) : base(Target)
	{
		IncludeOrderVersion = EngineIncludeOrderVersion.Latest;
		BuildEnvironment = TargetBuildEnvironment.UniqueIfNeeded;
		LinkType = TargetLinkType.Modular;
		LaunchModuleName = "ShaderBuildWorker";

		bCompileAgainstApplicationCore = true;

		// Force all shader formats to be built and included.
		bForceBuildShaderFormats = true;

		if (bShaderCompilerWorkerTrace)
		{
			GlobalDefinitions.Add("LLM_ENABLED_IN_CONFIG=1");
			GlobalDefinitions.Add("UE_MEMORY_TAGS_TRACE_ENABLED=1");
			bEnableTrace = true;
		}
	}
}
